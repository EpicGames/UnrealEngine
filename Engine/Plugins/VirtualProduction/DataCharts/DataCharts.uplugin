{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Data Charts",
	"Description": "Generate charts based on data tables",
	"Category": "Misc",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "DataCharts",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DataChartsEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	]
}
