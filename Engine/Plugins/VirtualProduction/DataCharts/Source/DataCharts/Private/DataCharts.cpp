// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataCharts.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FDataChartsModule"

void FDataChartsModule::StartupModule() {}

void FDataChartsModule::ShutdownModule() {}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FDataChartsModule, DataCharts)
