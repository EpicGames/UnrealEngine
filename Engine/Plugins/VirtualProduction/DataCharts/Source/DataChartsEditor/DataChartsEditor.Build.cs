// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DataChartsEditor : ModuleRules
{
    public DataChartsEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
				"Core",
                "CoreUObject",
				"EditorFramework",
                "Engine",
                "Projects",
                "Slate",
                "SlateCore",
                "UnrealEd",
                "PlacementMode",
            }
            );
    }
}
