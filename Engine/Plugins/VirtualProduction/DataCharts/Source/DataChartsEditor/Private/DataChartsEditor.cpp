// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataChartsEditor.h"
#include "DataChartsPlacement.h"
#include "DataChartsStyle.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FDataChartsEditorModule"

void FDataChartsEditorModule::StartupModule()
{
	FDataChartsStyle::Initialize();
	FDataChartsPlacement::RegisterPlacement();
}

void FDataChartsEditorModule::ShutdownModule()
{
	FDataChartsStyle::Shutdown();
	FDataChartsPlacement::UnregisterPlacement();
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FDataChartsEditorModule, DataChartsEditor)
