// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataChartsPlacement.h"
#include "Engine/Blueprint.h"
#include "IPlacementModeModule.h"
#include "DataChartsStyle.h"

#include "ActorFactories/ActorFactoryBlueprint.h"

#define LOCTEXT_NAMESPACE "FDataChartsEditorModule"

void FDataChartsPlacement::RegisterPlacement()
{
	IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
	FName CategoryName = "DataCharts";
	FPlacementCategoryInfo Info(LOCTEXT("PlacementMode_DataCharts", "Data Charts"), 
		FSlateIcon(FDataChartsStyle::Get()->GetStyleSetName(), "Icons.VirtualProduction"),
		CategoryName, TEXT("PMDataCharts"),45);
	PlacementModeModule.RegisterPlacementCategory(Info);

	TMap<const FName, UBlueprint*> MCharts;
	MCharts.Add(FName("DataCharts.BarIcon"), Cast<UBlueprint>(FSoftObjectPath(TEXT("/DataCharts/Blueprints/BP_BarChart.BP_BarChart")).TryLoad()) );
	MCharts.Add(FName("DataCharts.PieIcon"), Cast<UBlueprint>(FSoftObjectPath(TEXT("/DataCharts/Blueprints/BP_PieChart.BP_PieChart")).TryLoad()) );
	MCharts.Add(FName("DataCharts.LineIcon"), Cast<UBlueprint>(FSoftObjectPath(TEXT("/DataCharts/Blueprints/BP_LineChart.BP_LineChart")).TryLoad()) );

	for (TMap<const FName, UBlueprint*>::TConstIterator It = MCharts.CreateConstIterator(); It; ++It)
	{
		UBlueprint* ChartBP = It.Value();
		if (ChartBP == nullptr)
		{
			continue;
		}

		FString ChartName = ChartBP->GetName();
		ChartName.RemoveFromStart("BP_");

		FPlaceableItem* BPPlacement = new FPlaceableItem(
			*UActorFactoryBlueprint::StaticClass(),
			FAssetData(ChartBP, true),
			It.Key(),
			It.Key(),
			TOptional<FLinearColor>(),
			TOptional<int32>(),
			FText::FromString(ChartName)
		);

		IPlacementModeModule::Get().RegisterPlaceableItem(Info.UniqueHandle, MakeShareable(BPPlacement));
	}
}

void FDataChartsPlacement::UnregisterPlacement()
{
	if (IPlacementModeModule::IsAvailable())
	{
		IPlacementModeModule::Get().UnregisterPlacementCategory(TEXT("PMDataCharts"));
	}
}

#undef LOCTEXT_NAMESPACE
