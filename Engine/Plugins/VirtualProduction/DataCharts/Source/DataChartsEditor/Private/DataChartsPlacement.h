// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class FDataChartsPlacement
{
public:
	static void RegisterPlacement();
	static void UnregisterPlacement();
};
