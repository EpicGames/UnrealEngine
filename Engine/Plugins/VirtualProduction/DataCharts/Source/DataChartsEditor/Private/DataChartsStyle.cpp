// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataChartsStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"

TSharedPtr<FSlateStyleSet> FDataChartsStyle::StyleSet;

void FDataChartsStyle::Initialize()
{
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShared<FSlateStyleSet>(FName("DataChartsStyle"));

	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	FString IconFolder = IPluginManager::Get().FindPlugin(TEXT("DataCharts"))->GetBaseDir();
	FString IconPathBar = IconFolder + TEXT("/Resources/Bar.png");
	FString IconPathPie = IconFolder + TEXT("/Resources/Pie.png");
	FString IconPathLine = IconFolder + TEXT("/Resources/Line.png");
	FString VPIcon = IconFolder + TEXT("/Resources/VirtualProduction.svg");
	StyleSet->Set("DataCharts.BarIcon", new FSlateImageBrush(IconPathBar, FVector2D(40.0f, 40.0f)));
	StyleSet->Set("DataCharts.PieIcon", new FSlateImageBrush(IconPathPie, FVector2D(40.0f, 40.0f)));
	StyleSet->Set("DataCharts.LineIcon", new FSlateImageBrush(IconPathLine, FVector2D(40.0f, 40.0f)));
	StyleSet->Set("Icons.VirtualProduction", new FSlateVectorImageBrush(VPIcon, FVector2D(16.f, 16.f)));


	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}

void FDataChartsStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}
