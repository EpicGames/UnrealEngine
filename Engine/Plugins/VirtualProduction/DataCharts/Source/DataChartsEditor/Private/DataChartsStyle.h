// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class ISlateStyle;

class FDataChartsStyle
{
public:
	static void Initialize();
	static void Shutdown();
	static TSharedPtr<ISlateStyle> Get() { return StyleSet; }

private:
	static TSharedPtr<FSlateStyleSet> StyleSet;
};
